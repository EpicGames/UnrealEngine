{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ConversationToolset",
	"Description": "Toolset for Conversation Systems",
	"Category": "Other",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EditorOnly": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "ConversationToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "CommonConversation",
			"Enabled": true
		},
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
