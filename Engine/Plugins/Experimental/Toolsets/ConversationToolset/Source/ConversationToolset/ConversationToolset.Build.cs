// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ConversationToolset : ModuleRules
{
	public ConversationToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		PublicDependencyModuleNames.Add("Core");
	}
}
