// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationToolset.h"

#include "Modules/ModuleManager.h"

/*virtual*/ void FConversationToolsetModule::StartupModule()
{
}

/*virtual*/ void FConversationToolsetModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FConversationToolsetModule, ConversationToolset);
DEFINE_LOG_CATEGORY(LogConversationToolset);
